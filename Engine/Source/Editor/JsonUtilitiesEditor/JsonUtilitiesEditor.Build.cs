// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class JsonUtilitiesEditor : ModuleRules
{
    public JsonUtilitiesEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "Core",
		        "CoreUObject",
		        "Engine",
		        "UnrealEd",
		        "Slate",
		        "SlateCore",
		        "Json",
		        "JsonUtilities",
		        "BlueprintGraph",
		        "AssetTools",
	        }
        );
       
        PrivateIncludePaths.Add(
	        System.IO.Path.Combine(GetModuleDirectory("JsonUtilities"), "Private")
	    );
    }
}