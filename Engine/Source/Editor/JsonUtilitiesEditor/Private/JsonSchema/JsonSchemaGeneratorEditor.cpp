// Copyright Epic Games, Inc. All Rights Reserved. 

#include "JsonSchema/JsonSchemaGeneratorEditor.h"

#include "EdGraphSchema_K2.h"
#include "ObjectTools.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/StrProperty.h"
#include "UObject/Utf8StrProperty.h"
#include "UObject/PropertyOptional.h"
#include "Misc/ScopeExit.h"
#include "JsonSchema/JsonSchemaVisitorStack.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"

using namespace UE::JsonSchema;

//
// Internal FJsonSchemaMetadataPropertyDefaults
//
// Used for setting function struct param property defaults in the Json schema metadata.
//

namespace FJsonSchemaMetadataPropertyDefaults
{
	static const FString ParameterDefaultStringErrorPhraseString = 
		"Unable to convert default string to default value for metadata"; 
	
	template <typename T>
	void GetSoftPathStringsFromStructMemory(const uint8* const StructMemory, 
		FString& AssetPathDebugString, 
		FString& PackageNameString,
		FString& AssetNameString,
		FString& SubPathString,
		bool& bIsNullSoftPath)
	{
		static_assert(std::is_same_v<T, FSoftObjectPath> || std::is_same_v<T, FSoftClassPath>,
			"GetSoftPathStringsFromStructMemory() - T must be FSoftObjectPath or FSoftClassPath.");
		
		const FSoftObjectPath* SoftObjectPath = reinterpret_cast<const T*>(StructMemory);
		AssetPathDebugString = SoftObjectPath->GetAssetPathString();
		PackageNameString = SoftObjectPath->GetLongPackageName();
		AssetNameString = SoftObjectPath->GetAssetName();
		SubPathString = SoftObjectPath->GetSubPathString();
		bIsNullSoftPath = SoftObjectPath->IsNull();
	}
	
	bool SetFunctionStructPropertyDefaultAsNamedNumbersInPropertyMetadata(
		const FString& FunctionDebugName, const FString& FunctionParamPropertyDebugName,
		const FString& ParameterDefaultString, 
		const TArray<FString>& NumberValueNames,
		const TSharedRef<FJsonObject>& OutPropertyMetadata)
	{
		static const FString Delimiter = TEXT(",");
		static const FString DelimiterErrorPhraseTemplate = TEXT("(when delimited by \"") + Delimiter + TEXT("\")");

		TArray<FString> Parts;
		ParameterDefaultString.ParseIntoArray(Parts, *Delimiter);

		if (Parts.Num() != NumberValueNames.Num())
		{
			UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". " 
				"%ls.\n"
				"Reason: Default string contains %d values, but %d values were expected.",
				*FunctionDebugName, *FunctionParamPropertyDebugName,
				*ParameterDefaultStringErrorPhraseString,
				Parts.Num(), NumberValueNames.Num());
			
			return false;
		}

		const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		for (int32 I = 0; I < Parts.Num(); I++)
		{
			const FString NumericString = Parts[I].TrimStartAndEnd();
			
			if (double Value = 0.0;
				LexTryParseString(Value, *NumericString))
			{
				JsonObject->SetNumberField(*NumberValueNames[I], Value);
			}				
			else
			{
				UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". " 
					"%ls.\n"
					"Reason: Parameter default string contains non-double numeric string \"%ls\".\n"
					"Default string: \"%ls\"",
					*FunctionDebugName, *FunctionParamPropertyDebugName,
					*ParameterDefaultStringErrorPhraseString,
					*NumericString, 
					*ParameterDefaultString);
	
				return false;
			}
		}
		OutPropertyMetadata->SetObjectField(TEXT("default"), JsonObject);
		
		return true;
	}
	
	bool SetFunctionStructPropertyDefaultInPropertyMetadata(
		const FString& FunctionDebugName,
		const FStructProperty* StructProperty, const FString& ParameterDefaultString, 
		const TSharedRef<FJsonObject>& OutPropertyMetadata)
	{	
		// This is an struct property. This needs to become a JSON object.
		if (const UScriptStruct* ScriptStruct = StructProperty->Struct)
		{
			const int32 StructMemorySize = ScriptStruct->GetStructureSize(); 
			uint8* StructMemory = static_cast<uint8*>(FMemory_Alloca_Aligned(StructMemorySize, 
				ScriptStruct->GetMinAlignment()));
			FMemory::Memzero(StructMemory, StructMemorySize);
			
			ScriptStruct->InitializeStruct(StructMemory);
			
			ON_SCOPE_EXIT
			{
				ScriptStruct->DestroyStruct(StructMemory);
			};
			
			if (StructProperty->ImportText_Direct(*ParameterDefaultString, StructMemory, 
				nullptr, PPF_None))
			{
				// 'Import text' process succeeded, and we can convert the struct to Json directly.
				// But we also need special handling for some particular struct types.
				
				if (ScriptStruct == TBaseStructure<FSoftObjectPath>::Get() ||
					ScriptStruct == TBaseStructure<FSoftClassPath>::Get())
				{
					// Special handling of FSoftObjectPath/FSoftClassPath, to prevent getting the 
					// specialized default, which is a single combined string.
					// NOTE - We initialize from raw memory, so we can't safely use type
					// punning between FSoftObjectPath and derived FSoftClassPath, due to 
					// potential differences in memory layout.
					
					FString AssetPathDebugString, PackageNameString, AssetNameString, SubPathString;
					bool bIsNullSoftPath = false;
					if (ScriptStruct == TBaseStructure<FSoftObjectPath>::Get())
					{
						GetSoftPathStringsFromStructMemory<FSoftObjectPath>(StructMemory, 
							AssetPathDebugString, PackageNameString, AssetNameString, SubPathString, 
							bIsNullSoftPath);
					}
					else
					{
						GetSoftPathStringsFromStructMemory<FSoftClassPath>(StructMemory, 
							AssetPathDebugString, PackageNameString, AssetNameString, SubPathString, 
							bIsNullSoftPath);
					}
					
					if (bIsNullSoftPath || (!PackageNameString.IsEmpty() && !AssetNameString.IsEmpty()))
					{
						const TSharedPtr<FJsonObject> PackageAssetJsonObject = MakeShared<FJsonObject>();
						PackageAssetJsonObject->SetStringField(TEXT("packageName"), PackageNameString);
						PackageAssetJsonObject->SetStringField(TEXT("assetName"), AssetNameString);
						
						const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
						JsonObject->SetObjectField(TEXT("assetPath"), PackageAssetJsonObject);
						JsonObject->SetStringField(TEXT("subPathString"), SubPathString);
						OutPropertyMetadata->SetObjectField(TEXT("default"), JsonObject);

						return true;
					}
					else
					{
						UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". "
							"%ls.\n"
							"Reason: Property is non-empty soft object or class path struct property, "
							"but asset path \"%ls\" is missing package name \"%ls\" or asset name \"%ls\".\n",
							*FunctionDebugName, 
							*StructProperty->GetName(), 
							*ParameterDefaultStringErrorPhraseString,
							*AssetPathDebugString, 
							*PackageNameString, 
							*AssetNameString);

						return false;
					}
				}
				else
				{
					// All other struct types can use standard Struct->JsonObject conversion.
					
					if (const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
						FJsonObjectConverter::UStructToJsonObject(
							ScriptStruct, StructMemory, JsonObject.ToSharedRef()))
					{
						OutPropertyMetadata->SetObjectField(TEXT("default"), JsonObject);

						return true;
					}
					else
					{
						UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". "
							"%ls.\n"
							"Reason: Property is a struct property, "
							"but property's ScriptStruct type \"%ls\" could not be converted to a JSON object.\n",
							*FunctionDebugName, 
							*StructProperty->GetName(), 
							*ParameterDefaultStringErrorPhraseString,
							*ScriptStruct->GetPathName());
						return false;
					}
				}
			}
			else
			{
				// ImportText_Direct() failed, usually because the parameter default string is a simple 
				// list of numbers, which some properties like FVector will yield. For example, 
				// FVector's default string will be something like "1.00,2.00,3.00", which 
				// ImportText_Direct() can't handle (see above.) We still want these sorts of defaults 
				// to become JSON objects with named values, so we have to handle these parameter default
				// string manually.
				// NOTE - FTransform does not need to be handled here, it works with ImportText_Direct() above.
				
				TArray<FString> NumberValueNames;
				
				if (ScriptStruct == TBaseStructure<FVector>::Get() ||
					ScriptStruct == TBaseStructure<FIntVector>::Get())
				{
					NumberValueNames = {"x", "y", "z"}; 
				}
				else if (ScriptStruct == TBaseStructure<FVector2D>::Get() ||
					ScriptStruct == TBaseStructure<FIntPoint>::Get())
				{
					NumberValueNames = {"x", "y"}; 
				}
				else if (ScriptStruct == TBaseStructure<FVector4>::Get() ||
					ScriptStruct == TBaseStructure<FQuat>::Get())
				{
					NumberValueNames = {"x", "y", "z", "w"}; 
				}
				else if (ScriptStruct == TBaseStructure<FRotator>::Get())
				{
					NumberValueNames = {"pitch", "yaw", "roll"}; 
				}
				else if (ScriptStruct == TBaseStructure<FLinearColor>::Get() ||
					ScriptStruct == TBaseStructure<FColor>::Get())
				{
					NumberValueNames = {"r", "g", "b", "a"}; 
				}
				else
				{
					UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". "
						"%ls.\n"
						"Reason: Property is a struct property, "
						"but property's ScriptStruct type \"%ls\" is currently unsupported.",
						*FunctionDebugName, 
						*StructProperty->GetName(), 
						*ParameterDefaultStringErrorPhraseString, 
						*ScriptStruct->GetPathName());

					return false;
				}
				
				return SetFunctionStructPropertyDefaultAsNamedNumbersInPropertyMetadata(
					FunctionDebugName, StructProperty->GetName(),
					ParameterDefaultString, NumberValueNames, OutPropertyMetadata);
			}
		}
		else
		{
			UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". "
				"%ls.\n"
				"Reason: Property is struct property, but has no underlying ScriptStruct. "
				"It may be part of an unloaded module, or reflection data may be incomplete.",
				*FunctionDebugName, 
				*StructProperty->GetName(), 
				*ParameterDefaultStringErrorPhraseString);
			
			return false;
		}
	}	

	void SetFunctionPropertyDefaultInPropertyMetadata(
		const UFunction* PropertyOwnerFunction, const FProperty* Property, 
		const FJsonSchemaPropertyFilter& PropertyFilter, const TSharedRef<FJsonObject>& OutPropertyMetadata)
	{
		if (Property->HasAnyPropertyFlags(CPF_ReturnParm))
		{
			// Return values don't have defaults.
			return;
		}
		
		const FString FunctionDebugName = PropertyOwnerFunction->GetName();
		
		FString ParameterDefaultString;
		UEdGraphSchema_K2::FindFunctionParameterDefaultValue(PropertyOwnerFunction, Property, ParameterDefaultString);
		if (!ParameterDefaultString.IsEmpty())
		{
			if (PropertyFilter.CustomCb && PropertyFilter.CustomCb->IsBound())
			{
				if (PropertyFilter.CustomCb->Execute(Property, ParameterDefaultString, OutPropertyMetadata))
				{
					return;
				}
			}

			if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				if (const UEnum* Enum = EnumProperty->GetEnum())
				{
					if (int64 Value = 0;
						LexTryParseString(Value, *ParameterDefaultString))
					{
						const FString Name = Enum->GetNameStringByValue(Value);
						OutPropertyMetadata->SetStringField(TEXT("default"), (Name.IsEmpty() ? ParameterDefaultString : Name));
					}
					else
					{
						OutPropertyMetadata->SetStringField(TEXT("default"), ParameterDefaultString);
					}
				}
				else
				{
					UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". "
						"%ls.\n"
						"Reason: Property is enum property, but has no underlying enum field. "
						"It may be part of an unloaded module, or reflection data may be incomplete.",
						*FunctionDebugName, 
						*Property->GetName(), 
						*ParameterDefaultStringErrorPhraseString);
				}
			}
			else if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
			{
				if (const UEnum* Enum = NumericProperty->GetIntPropertyEnum())
				{
					if (int64 Value = 0;
						LexTryParseString(Value, *ParameterDefaultString))
					{
						const FString Name = Enum->GetNameStringByValue(Value);
						OutPropertyMetadata->SetStringField(TEXT("default"), (Name.IsEmpty() ? ParameterDefaultString : Name));
					}
					else
					{
						UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". "
							"%ls.\n"
							"Reason: Property is numeric enum property, "
							"but the default string for the Property is not integer type "
							"and cannot be resolved into an Enum entry.\n"
							"Default string: \"%ls\"",
							*FunctionDebugName, 
							*Property->GetName(), 
							*ParameterDefaultStringErrorPhraseString,
							*ParameterDefaultString);
					}
				}
				else
				{
					// This is a normal numeric property. Convert default string to double (ints will end up being rounded in the JSON.)
					OutPropertyMetadata->SetNumberField(TEXT("default"), FCString::Atod(*ParameterDefaultString));
				}
			}
			else if (Property->IsA<FBoolProperty>())
			{
				// Default string will be either 'true' or 'false' only, and needs to become a bool.
				OutPropertyMetadata->SetBoolField(TEXT("default"), (ParameterDefaultString.Compare(TEXT("true"), ESearchCase::IgnoreCase) == 0));
			}
			else if (Property->IsA<FStrProperty>() || Property->IsA<FNameProperty>() || Property->IsA<FTextProperty>())
			{
				// Default is one of the string properties, and we can use the default as-is.
				OutPropertyMetadata->SetStringField(TEXT("default"), ParameterDefaultString);
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				// This is an struct property. This needs to become a JSON object, which 
				// requires some special handling.
				if (!SetFunctionStructPropertyDefaultInPropertyMetadata(FunctionDebugName, StructProperty, ParameterDefaultString, OutPropertyMetadata))
				{
					UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". " 
						"%ls.\n"
						"Default string: \"%ls\"",
						*FunctionDebugName, 
						*Property->GetName(), 
						*ParameterDefaultStringErrorPhraseString,
						*ParameterDefaultString);
				}
			}
			else if (Property->IsA<FObjectProperty>())
			{
				if (ParameterDefaultString == "None")
				{
					OutPropertyMetadata->SetField(TEXT("default"), MakeShared<FJsonValueNull>());
				}
				else
				{
					OutPropertyMetadata->SetStringField(TEXT("default"), ParameterDefaultString);
				}
			}
			else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
			{
				// First see if there is a default based on the property the optional contains.
				SetFunctionPropertyDefaultInPropertyMetadata(
					PropertyOwnerFunction, OptionalProperty->GetValueProperty(), PropertyFilter, OutPropertyMetadata);
				// If that fails, make sure there is a default and that it's empty/none.
				if (!(OutPropertyMetadata->HasField(TEXT("default"))))
				{
					OutPropertyMetadata->SetField(TEXT("default"), MakeShared<FJsonValueNull>());
				}
			}
			else
			{
				UE_LOGF(LogJson, Error, "Function \"%ls\", param property \"%ls\". " 
					"%ls.\n"
					"Reason: Property type \"%ls\" is not recognized.\n"
					"Default string: \"%ls\"",
					*FunctionDebugName, 
					*Property->GetName(), 
					*ParameterDefaultStringErrorPhraseString,
					*Property->GetClass()->GetName(),
					*ParameterDefaultString);
			}
		}
	}
};

//
// Internal FJsonSchemaMetadataVisitor
//

class FJsonSchemaMetadataVisitor
{
public:

	explicit FJsonSchemaMetadataVisitor(FJsonSchemaEditorMetadata& InOutputEditorMetadata,
			const FJsonSchemaPropertyFilter& InPropertyFilter = FJsonSchemaPropertyFilter()) :
		OutputEditorMetadata(InOutputEditorMetadata)
	{
	}
		
	void VisitRoot(const FVisitorTarget& VisitorTarget, 
		const FJsonSchemaMemberPath& MemberPath, const FJsonSchemaPropertyFilter& PropertyFilter,
		const void* InstanceMemory)
	{
		const EVisitorStackElementFlags Flags = (VisitorTarget.IsType<FValidConstUStruct>() ? 
			EVisitorStackElementFlags::IsRootStruct : EVisitorStackElementFlags::None);
    
		VisitorStack.Empty(); // Precaution.

		const FVisitorStackElement RootStackElement(VisitorTarget, Flags, MemberPath, PropertyFilter, nullptr, InstanceMemory);
		VisitorStack.Add(RootStackElement);
		
		while (!VisitorStack.IsEmpty())
		{
			const FVisitorStackElement StackElement = VisitorStack.Pop();
			
			if (StackElement.VisitorTarget.IsType<FValidConstUStruct>())
			{
				VisitUStruct(StackElement.VisitorTarget.Get<FValidConstUStruct>(),
					StackElement.MemberPath, StackElement.PropertyFilter,
					StackElement.Flags, StackElement.InstanceMemory);
			}
			else if (StackElement.VisitorTarget.IsType<FValidConstFProperty>())
			{
				VisitFProperty(StackElement.VisitorTarget.Get<FValidConstFProperty>(),
					StackElement.MemberPath, StackElement.PropertyFilter, StackElement.Flags, StackElement.InstanceMemory);
			}
		}
	}

	void VisitFProperty(const FValidConstFProperty Property, 
		const FJsonSchemaMemberPath& MemberPath, const FJsonSchemaPropertyFilter& PropertyFilter,
		const EVisitorStackElementFlags Flags, const void* InstanceMemory)
	{
		OutputEditorMetadata.CurrentPropertyMemberPath = MemberPath;
		
		const UStruct* PropertyOwnerStruct = Property->GetOwnerStruct();
		const UFunction* PropertyOwnerFunction = Cast<UFunction>(PropertyOwnerStruct);
		
		// Check to see if we should ignore this property.
		if (PropertyFilter.IsPropertyIgnored(Property))
		{
			return;
		}
		
		const TSharedRef<FJsonObject> PropertyMetadata = MakeShared<FJsonObject>();
		
		const FString PropertyName = Property->GetAuthoredName();
		
		if (const FText PropertyToolTip = Property->GetToolTipText(); 
			!PropertyToolTip.IsEmpty())
		{
			PropertyMetadata->SetStringField(TEXT("description"), PropertyToolTip.ToString());
		}
		else if (PropertyOwnerFunction)
		{
			const FString FunctionParamTooltip = (Property->HasAnyPropertyFlags(CPF_ReturnParm) ?
				ObjectTools::GetDefaultTooltipForFunctionReturn(PropertyOwnerFunction) :
				ObjectTools::GetDefaultTooltipForFunctionParam(PropertyOwnerFunction, PropertyName));
			if (!FunctionParamTooltip.IsEmpty())
			{
				PropertyMetadata->SetStringField(TEXT("description"), FunctionParamTooltip);
			}
		}

		// Add minimum/maximum for numeric properties.
		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
		{
			if (NumericProperty->HasMetaData("ClampMin"))
			{
				PropertyMetadata->SetNumberField(TEXT("minimum"), NumericProperty->GetDoubleMetaData("ClampMin"));
			}
			if (NumericProperty->HasMetaData("ClampMax"))
			{
				PropertyMetadata->SetNumberField(TEXT("maximum"), NumericProperty->GetDoubleMetaData("ClampMax"));
			}
		}

		// If function, add any parameter defaults.
		if (PropertyOwnerFunction)
		{
			FJsonSchemaMetadataPropertyDefaults::SetFunctionPropertyDefaultInPropertyMetadata(PropertyOwnerFunction, Property, PropertyFilter, PropertyMetadata);
		}
		
		if (PropertyMetadata->Values.Num())
		{
			OutputEditorMetadata.SetPropertyMetadataForCurrentPropertyMemberPath(PropertyMetadata);
		}

		// Recurse into inner properties.
		// See all NOTE_JSON_SCHEMA_CHECK_FLAGS_NO_PARMS.
		FJsonSchemaPropertyFilter NewPropertyFilter = PropertyFilter;
		NewPropertyFilter.CheckFlags &= ~CPF_ParmFlags;

		if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			// FInstancedStruct and FInstancedPropertyBag require runtime dispatch: the actual struct
			// type and its memory are resolved from the live instance. Regular structs pass through.
			const UStruct* InnerStruct = nullptr;
			const void* InnerInstanceMemory = nullptr;
			ResolveStructForRecursion(StructProp->Struct, InstanceMemory, InnerStruct, InnerInstanceMemory);
			if (InnerStruct)
			{
				VisitorStack.Emplace(
					FVisitorTarget(TInPlaceType<FValidConstUStruct>(), InnerStruct),
					EVisitorStackElementFlags::None, OutputEditorMetadata.CurrentPropertyMemberPath,
					NewPropertyFilter, nullptr, InnerInstanceMemory);
			}
		}
		else if (const FOptionalProperty* OptProp = CastField<FOptionalProperty>(Property))
		{
			// Unwrap the optional to recurse into its inner struct type, if any.
			if (const FStructProperty* InnerStructProp = CastField<FStructProperty>(OptProp->GetValueProperty()))
			{
				// Get the value pointer only if the optional is set; nullptr otherwise (the
				// downstream ResolveStructForRecursion will safely skip instanced types with no instance).
				const void* OptValueMemory = InstanceMemory
					? OptProp->GetValuePointerForReadIfSet(InstanceMemory)
					: nullptr;
				const UStruct* InnerStruct = nullptr;
				const void* InnerInstanceMemory = nullptr;
				ResolveStructForRecursion(InnerStructProp->Struct, OptValueMemory, InnerStruct, InnerInstanceMemory);
				if (InnerStruct)
				{
					VisitorStack.Emplace(
						FVisitorTarget(TInPlaceType<FValidConstUStruct>(), InnerStruct),
						EVisitorStackElementFlags::None, OutputEditorMetadata.CurrentPropertyMemberPath,
						NewPropertyFilter, nullptr, InnerInstanceMemory);
				}
			}
		}
		else if (const UStruct* CollectionInnerStruct = GetFPropertyInnerStruct(*Property))
		{
			// Array/Set/Map inner struct types. No representative instance memory is available
			// (elements may differ at runtime), so metadata is collected statically.
			VisitorStack.Emplace(
				FVisitorTarget(TInPlaceType<FValidConstUStruct>(), CollectionInnerStruct),
				EVisitorStackElementFlags::None, OutputEditorMetadata.CurrentPropertyMemberPath,
				NewPropertyFilter, nullptr, nullptr);
		}
	}
	
	void VisitUStruct(const FValidConstUStruct Struct, 
		const FJsonSchemaMemberPath& MemberPath, const FJsonSchemaPropertyFilter& PropertyFilter,
		const EVisitorStackElementFlags Flags, const void* InstanceMemory)
	{
		OutputEditorMetadata.CurrentPropertyMemberPath = MemberPath;
		
		const UFunction* StructAsFunction = Cast<UFunction>(Struct);
		
		if (EnumHasAnyFlags(Flags, EVisitorStackElementFlags::IsRootStruct))
		{
			OutputEditorMetadata.RootStructMetadata = MakeShared<FJsonObject>();
			
			if (const FString Description = (
					StructAsFunction ?
					ObjectTools::GetDefaultTooltipForFunction(StructAsFunction, false) :
					Struct->GetToolTipText().ToString());
				!Description.IsEmpty())
			{
				OutputEditorMetadata.RootStructMetadata->SetStringField(TEXT("description"), Description);
			}
		}
		
		for (const FProperty* Property : TFieldRange<FProperty>(Struct))
		{
			const FString PropertyName = Property->GetAuthoredName();

			UE_JSON_SCHEMA_SCOPED_MEMBER_PATH_PUSH(OutputEditorMetadata.CurrentPropertyMemberPath, PropertyName,
				!EnumHasAnyFlags(Flags, EVisitorStackElementFlags::IsRootStruct));

			const void* PropertyInstanceMemory = InstanceMemory
				? static_cast<const uint8*>(InstanceMemory) + Property->GetOffset_ForInternal()
				: nullptr;

			VisitorStack.Emplace(
				FVisitorTarget(TInPlaceType<FValidConstFProperty>(), Property),
				EVisitorStackElementFlags::None, OutputEditorMetadata.CurrentPropertyMemberPath, PropertyFilter, nullptr, PropertyInstanceMemory);
		}
	}
	
private:

	// Resolves the inner struct type and instance memory to use for metadata recursion.
	// For FInstancedStruct and FInstancedPropertyBag, the actual struct type is only known at
	// runtime; OutStruct is set to nullptr (skip recursion) when no valid instance is available.
	// For all other structs, OutStruct is the static struct type and OutInstanceMemory is passed through.
	static void ResolveStructForRecursion(const UScriptStruct* StaticStruct, const void* InstanceMemory,
		const UStruct*& OutStruct, const void*& OutInstanceMemory)
	{
		OutStruct = nullptr;
		OutInstanceMemory = nullptr;

		if (StaticStruct == FInstancedStruct::StaticStruct())
		{
			if (InstanceMemory)
			{
				const FInstancedStruct* InstancedStruct = static_cast<const FInstancedStruct*>(InstanceMemory);
				if (InstancedStruct->IsValid())
				{
					OutStruct = InstancedStruct->GetScriptStruct();
					OutInstanceMemory = InstancedStruct->GetMemory();
				}
			}
			return;
		}

		if (StaticStruct == FInstancedPropertyBag::StaticStruct())
		{
			if (InstanceMemory)
			{
				const FInstancedPropertyBag* Bag = static_cast<const FInstancedPropertyBag*>(InstanceMemory);
				if (Bag->IsValid())
				{
					if (const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct())
					{
						OutStruct = BagStruct;
						OutInstanceMemory = Bag->GetValue().GetMemory();
					}
				}
			}
			return;
		}

		OutStruct = StaticStruct;
		OutInstanceMemory = InstanceMemory;
	}

	// Returns the inner struct type for collection properties (Array/Set/Map), or nullptr.
	// FStructProperty and FOptionalProperty are handled directly in VisitFProperty via
	// ResolveStructForRecursion so that instance memory can be propagated correctly.
	const UStruct* GetFPropertyInnerStruct(const FProperty& Property) const
	{
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(&Property))
		{
			if (const FStructProperty* InnerStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
			{
				return InnerStructProperty->Struct;
			}
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(&Property))
		{
			if (const FStructProperty* ElementStructProperty = CastField<FStructProperty>(SetProperty->ElementProp))
			{
				return ElementStructProperty->Struct;
			}
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(&Property))
		{
			if (MapProperty->KeyProp->IsA<FStrProperty>())
			{
				if (const FStructProperty* ValueStructProperty = CastField<FStructProperty>(MapProperty->ValueProp))
				{
					return ValueStructProperty->Struct;
				}
			}
		}
		return nullptr;
	}
	
	FVisitorStack VisitorStack;
	FJsonSchemaEditorMetadata& OutputEditorMetadata; 
};

//
// FJsonSchemaGeneratorEditor
//

/*static*/ FJsonSchemaEditorMetadata FJsonSchemaGeneratorEditor::FPropertyToJsonSchemaMetadata(TNotNull<const FProperty*> Property,
	const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory)
{
	FJsonSchemaEditorMetadata EditorMetadata;
	
	FJsonSchemaMetadataVisitor JsonSchemaMetadataVisitor(EditorMetadata, PropertyFilter);
	JsonSchemaMetadataVisitor.VisitRoot(
		FVisitorTarget(TInPlaceType<FValidConstFProperty>(), Property), 
		FJsonSchemaMemberPath(*Property->GetAuthoredName()),
		PropertyFilter,
		InstanceMemory);
	
	return EditorMetadata;
}

/*static*/ FJsonSchemaEditorMetadata FJsonSchemaGeneratorEditor::UStructToJsonSchemaMetadata(TNotNull<const UStruct*> Struct,
	const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory)
{
	FJsonSchemaEditorMetadata EditorMetadata;
	
	FJsonSchemaMetadataVisitor JsonSchemaMetadataVisitor(EditorMetadata, PropertyFilter);
	JsonSchemaMetadataVisitor.VisitRoot(
		FVisitorTarget(TInPlaceType<FValidConstUStruct>(), Struct), 
		FJsonSchemaMemberPath(),
		PropertyFilter,
		InstanceMemory);
	
	return EditorMetadata;
}
