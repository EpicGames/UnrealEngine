// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonUtilitiesEditorModularFeature.h"

#include "JsonSchema/JsonSchemaGeneratorEditor.h"

/*virtual*/ FJsonSchemaEditorMetadata FJsonUtilitiesEditorModularFeature::FPropertyToJsonSchemaMetadata(TNotNull<const FProperty*> Property,
	const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory) const
{
	return FJsonSchemaGeneratorEditor::FPropertyToJsonSchemaMetadata(Property, PropertyFilter, InstanceMemory);
}

/*virtual*/ FJsonSchemaEditorMetadata FJsonUtilitiesEditorModularFeature::UStructToJsonSchemaMetadata(TNotNull<const UStruct*> Struct,
	const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory) const
{
	return FJsonSchemaGeneratorEditor::UStructToJsonSchemaMetadata(Struct, PropertyFilter, InstanceMemory);
}

