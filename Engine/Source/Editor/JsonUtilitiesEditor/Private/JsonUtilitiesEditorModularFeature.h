// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JsonUtilitiesModularFeature.h"

/**
 * Derived class for Modular Feature - Editor functionality that the Runtime module will call when in Editor, to collect metadata.
 * Registered in Editor module.
 */
class FJsonUtilitiesEditorModularFeature : public IJsonUtilitiesModularFeature
{
public:

	JSONUTILITIESEDITOR_API virtual FJsonSchemaEditorMetadata FPropertyToJsonSchemaMetadata(TNotNull<const FProperty*> Property,
		const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory) const override;

	JSONUTILITIESEDITOR_API virtual	FJsonSchemaEditorMetadata UStructToJsonSchemaMetadata(TNotNull<const UStruct*>,
		const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory) const override;
};
