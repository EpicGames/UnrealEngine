// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "JsonUtilitiesEditorModularFeature.h"

class FJsonUtilitiesEditorModule : public IModuleInterface
{
public:

	TUniquePtr<FJsonUtilitiesEditorModularFeature> ModularFeature;
	
	virtual void StartupModule() override
	{
		FModuleManager::LoadModuleChecked<IModuleInterface>(TEXT("JsonUtilities"));
		
		ModularFeature = MakeUnique<FJsonUtilitiesEditorModularFeature>();
		IModularFeatures::Get().RegisterModularFeature(IJsonUtilitiesModularFeature::Name, ModularFeature.Get());
	}
	
	virtual void ShutdownModule() override
	{
		if (ModularFeature.IsValid())
		{
			IModularFeatures::Get().UnregisterModularFeature(IJsonUtilitiesModularFeature::Name, ModularFeature.Get());
			ModularFeature.Reset();
		}
	}
};

IMPLEMENT_MODULE(FJsonUtilitiesEditorModule, JsonUtilitiesEditor)
