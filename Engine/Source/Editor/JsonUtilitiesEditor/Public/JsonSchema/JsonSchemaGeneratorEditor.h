// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "Dom/JsonObject.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

#include "JsonSchema/JsonSchemaPropertyFilter.h"
#include "JsonSchema/JsonSchemaEditorMetadata.h"

class FJsonSchemaGeneratorEditor
{
public:
	
	/**
	 * Usually not called by a developer directly. Used by the Runtime module via the Module Feature.
	 * 
	 * @param Property The property to collect metadata for.
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param InstanceMemory When set allows specs to be generated from FInstancedStruct and FInstancedPropertyBag.
	 * @return The collected property metadata.
	 */
	JSONUTILITIESEDITOR_API static FJsonSchemaEditorMetadata FPropertyToJsonSchemaMetadata(TNotNull<const FProperty*> Property,
		const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory);

	/**
	 * Usually not called by a developer directly. Used by the Runtime module via the Module Feature.
	 * 
	 * Recursively walks Struct's property hierarchy, collecting editor-only meta-data like
	 * tooltips and other Meta values, returning them in the cookable metadata.
	 *
	 * e.g:
	 *
	 *		// Details for a person
	 *		USTRUCT()
	 *		struct FPerson
	 *		{
	 *			GENERATED_BODY()
	 *	
	 *			// Full name
	 *			UPROPERTY()
	 *			FString Name;
	 *
	 *			// How old they are
	 *			UPROPERTY(meta = (ClampMin = "1"))
	 *			int32 Age;
	 *		}
	 *
	 *		// A family of people
	 *		USTRUCT()
	 *		struct FFamily
	 *		{
	 *			// Members of this family
	 *			UPROPERTY()
	 *			TArray<FPerson> Members; 
	 *		}
	 *
	 *		-> metadata:
	 *			description -> "A family of people"
	 *			PropertyMetadata ->
	 *				Members = "Members of this family"
	 *				Members.Name = "Full name"
	 *				Members.Age = "How old they are" (ClampMin = 1)
	 *
	 * Note: When Struct is a UFunction, function parameter descriptions will be harvested from doxygen markup in the function comment e.g:
	 *
	 *		// Returns the best friend of the given person
	 *		// Person: The person to find the best friend of
	 * 		UFUNCTION()
	 * 		FPerson GetBestFriend(const FPerson& Person)
	 *
	 * 		-> metadata:
	 *			description -> "Returns the best friend of the given person"
	 *			PropertyMetadata -> Person -> "The person to find the best friend of"
	 *
	 * @param Struct The type to collect property data for. Can be either UScriptStruct, UClass or UFunction.
	 * @param PropertyFilter Determines how to filter collected struct and property data.
	 * @param InstanceMemory When set allows specs to be generated from FInstancedStruct and FInstancedPropertyBag.
	 * @return The collected property metadata.
	 */
	JSONUTILITIESEDITOR_API static FJsonSchemaEditorMetadata UStructToJsonSchemaMetadata(TNotNull<const UStruct*> Struct,
		const FJsonSchemaPropertyFilter& PropertyFilter, const void* InstanceMemory);
};
